// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioFormatADPCM : ModuleRules
{
	public AudioFormatADPCM(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");
		PrivateDependencyModuleNames.Add("AdpcmAudioDecoder");
		PrivateDependencyModuleNames.Add("Core");
		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("Engine");
			
		}
	}
}
