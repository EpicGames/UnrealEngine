// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioFormatADPCM.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "Serialization/MemoryWriter.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IAudioFormat.h"
#include "Interfaces/IAudioFormatModule.h"
#include "AudioDecompress.h"
#include "Audio.h"
#include "Decoders/ADPCMAudioInfo.h"

DEFINE_LOG_CATEGORY_STATIC(LogAudioFormatADPCM, Log, All);

#define UE_MAKEFOURCC(ch0, ch1, ch2, ch3)\
	((uint32)(uint8)(ch0) | ((uint32)(uint8)(ch1) << 8) |\
	((uint32)(uint8)(ch2) << 16) | ((uint32)(uint8)(ch3) << 24 ))

namespace
{
	struct RiffDataChunk
	{
		uint32 ID;
		uint32 DataSize;
		uint8* Data;
	};

	template <uint32 N>
	void GenerateWaveFile(RiffDataChunk(& RiffDataChunks)[N], TArray<uint8>& CompressedDataStore)
	{
		// 'WAVE'
		uint32 RiffDataSize = sizeof(uint32);

		// Determine the size of the wave file to be generated
		for (uint32 Scan = 0; Scan < N; ++Scan)
		{
			const RiffDataChunk& Chunk = RiffDataChunks[Scan];
			RiffDataSize += (sizeof(Chunk.ID) + sizeof(Chunk.DataSize) + Chunk.DataSize);
		}

		// Allocate space for the output data + 'RIFF' + ChunkSize
		uint32 OutputDataSize = RiffDataSize + sizeof(uint32) + sizeof(uint32);
		
		CompressedDataStore.Empty(OutputDataSize);
		FMemoryWriter CompressedData(CompressedDataStore);
		
		uint32 rID = UE_MAKEFOURCC('R','I','F','F');
		CompressedData.Serialize(&rID, sizeof(rID));
		CompressedData.Serialize(&RiffDataSize, sizeof(RiffDataSize));

		rID = UE_MAKEFOURCC('W','A','V','E');
		CompressedData.Serialize(&rID, sizeof(rID));

		// Write each sub-chunk to the output data
		for (uint32 Scan = 0; Scan < N; ++Scan)
		{
			RiffDataChunk& Chunk = RiffDataChunks[Scan];
			
			CompressedData.Serialize(&Chunk.ID, sizeof(Chunk.ID));
			CompressedData.Serialize(&Chunk.DataSize, sizeof(Chunk.DataSize));
			CompressedData.Serialize(Chunk.Data, Chunk.DataSize);
		}
	}

	template <typename T, uint32 B>
	inline T SignExtend(const T ValueToExtend)
	{
		struct { T ExtendedValue:B; } SignExtender;
		return SignExtender.ExtendedValue = ValueToExtend;
	}

	template <typename T>
	inline T ReadFromByteStream(const uint8* ByteStream, int32& ReadIndex, bool bLittleEndian = true)
	{
		T ValueRaw = 0;

		if (bLittleEndian)
		{
#if PLATFORM_LITTLE_ENDIAN
			for (int32 ByteIndex = 0; ByteIndex < sizeof(T); ++ByteIndex)
#else
			for (int32 ByteIndex = sizeof(T) - 1; ByteIndex >= 0; --ByteIndex)
#endif // PLATFORM_LITTLE_ENDIAN
			{
				ValueRaw |= ByteStream[ReadIndex++] << 8 * ByteIndex;
			}
		}
		else
		{
#if PLATFORM_LITTLE_ENDIAN
			for (int32 ByteIndex = sizeof(T) - 1; ByteIndex >= 0; --ByteIndex)
#else
			for (int32 ByteIndex = 0; ByteIndex < sizeof(T); ++ByteIndex)
#endif // PLATFORM_LITTLE_ENDIAN
			{
				ValueRaw |= ByteStream[ReadIndex++] << 8 * ByteIndex;
			}
		}

		return ValueRaw;
	}

	template <typename T>
	inline void WriteToByteStream(T Value, uint8* ByteStream, int32& WriteIndex, bool bLittleEndian = true)
	{
		if (bLittleEndian)
		{
#if PLATFORM_LITTLE_ENDIAN
			for (int32 ByteIndex = 0; ByteIndex < sizeof(T); ++ByteIndex)
#else
			for (int32 ByteIndex = sizeof(T) - 1; ByteIndex >= 0; --ByteIndex)
#endif // PLATFORM_LITTLE_ENDIAN
			{
				ByteStream[WriteIndex++] = (Value >> (8 * ByteIndex)) & 0xFF;
			}
		}
		else
		{
#if PLATFORM_LITTLE_ENDIAN
			for (int32 ByteIndex = sizeof(T) - 1; ByteIndex >= 0; --ByteIndex)
#else
			for (int32 ByteIndex = 0; ByteIndex < sizeof(T); ++ByteIndex)
#endif // PLATFORM_LITTLE_ENDIAN
			{
				ByteStream[WriteIndex++] = (Value >> (8 * ByteIndex)) & 0xFF;
			}
		}
	}

	template <typename T>
	inline T ReadFromArray(const T* ElementArray, int32& ReadIndex, int32 NumElements, int32 IndexStride = 1)
	{
		T OutputValue = 0;

		if (ReadIndex >= 0 && ReadIndex < NumElements)
		{
			OutputValue = ElementArray[ReadIndex];
			ReadIndex += IndexStride;
		}

		return OutputValue;
	}
} // end namespace

namespace LPCM
{
	void Encode(const TArray<uint8>& InputPCMData, TArray<uint8>& CompressedDataStore, const FSoundQualityInfo& QualityInfo)
	{
		WaveFormatHeader Format;
		Format.nChannels = static_cast<uint16>(QualityInfo.NumChannels);
		Format.nSamplesPerSec = QualityInfo.SampleRate;
		Format.nBlockAlign = static_cast<uint16>(Format.nChannels * sizeof(int16));
		Format.nAvgBytesPerSec = Format.nBlockAlign * QualityInfo.SampleRate;
		Format.wBitsPerSample = 16;
		Format.wFormatTag = WAVE_FORMAT_LPCM;
		Format.cbSize = 0;

		RiffDataChunk RiffDataChunks[2];
		RiffDataChunks[0].ID = UE_MAKEFOURCC('f','m','t',' ');
		RiffDataChunks[0].DataSize = sizeof(Format);
		RiffDataChunks[0].Data = reinterpret_cast<uint8*>(&Format);

		RiffDataChunks[1].ID = UE_MAKEFOURCC('d','a','t','a');
		RiffDataChunks[1].DataSize = InputPCMData.Num();
		RiffDataChunks[1].Data = const_cast<uint8*>(InputPCMData.GetData());

		GenerateWaveFile(RiffDataChunks, CompressedDataStore);
	}
} // end namespace LPCM

namespace ADPCM
{
	template <typename T>
	static void GetAdaptationTable(T(& OutAdaptationTable)[NUM_ADAPTATION_TABLE])
	{
		// Magic values as specified by standard
		static T AdaptationTable[] =
		{
			230, 230, 230, 230, 307, 409, 512, 614,
			768, 614, 512, 409, 307, 230, 230, 230
		};

		FMemory::Memcpy(&OutAdaptationTable, AdaptationTable, sizeof(AdaptationTable));
	}

	struct FAdaptationContext
	{
	public:
		// Adaptation constants
		int32 AdaptationTable[NUM_ADAPTATION_TABLE];
		int32 AdaptationCoefficient1[NUM_ADAPTATION_COEFF];
		int32 AdaptationCoefficient2[NUM_ADAPTATION_COEFF];

		int32 AdaptationDelta;
		int32 Coefficient1;
		int32 Coefficient2;
		int32 Sample1;
		int32 Sample2;

		FAdaptationContext() :
			AdaptationDelta(0),
			Coefficient1(0),
			Coefficient2(0),
			Sample1(0),
			Sample2(0)
		{
			GetAdaptationTable(AdaptationTable);
			GetAdaptationCoefficients(AdaptationCoefficient1, AdaptationCoefficient2);
		}
	};

	/**
	* Encodes a 16-bit PCM sample to a 4-bit ADPCM sample.
	**/
	uint8 EncodeNibble(FAdaptationContext& Context, int16 NextSample)
	{
		int32 PredictedSample = (Context.Sample1 * Context.Coefficient1 + Context.Sample2 * Context.Coefficient2) / 256;
		int32 ErrorDelta = (NextSample - PredictedSample) / Context.AdaptationDelta;
		ErrorDelta = FMath::Clamp(ErrorDelta, -8, 7);

		// Predictor must be clamped within the 16-bit range
		PredictedSample += (Context.AdaptationDelta * ErrorDelta);
		PredictedSample = FMath::Clamp(PredictedSample, -32768, 32767);

		int8 SmallDelta = static_cast<int8>(ErrorDelta);
		uint8 EncodedNibble = reinterpret_cast<uint8&>(SmallDelta) & 0x0F;

		// Shuffle samples for the next iteration
		Context.Sample2 = Context.Sample1;
		Context.Sample1 = static_cast<int16>(PredictedSample);
		Context.AdaptationDelta = (Context.AdaptationDelta * Context.AdaptationTable[EncodedNibble]) / 256;
		Context.AdaptationDelta = FMath::Max(Context.AdaptationDelta, 16);

		return EncodedNibble;
	}

	int32 EncodeBlock(const int16* InputPCMSamples, int32 SampleStride, int32 NumSamples, int32 BlockSize, uint8* EncodedADPCMData)
	{
		FAdaptationContext Context;
		int32 ReadIndex = 0;
		int32 WriteIndex = 0;

		/* TODO::JTM - Dec 10, 2012 05:30PM - Calculate the optimal starting coefficient */
		uint8 CoefficientIndex = 0;
		Context.AdaptationDelta = Context.AdaptationTable[0];
		// First PCM sample goes to Context.Sample2, decoder will reverse it
		Context.Sample2 = ReadFromArray<int16>(InputPCMSamples, ReadIndex, NumSamples, SampleStride);
		Context.Sample1 = ReadFromArray<int16>(InputPCMSamples, ReadIndex, NumSamples, SampleStride);
		Context.Coefficient1 = Context.AdaptationCoefficient1[CoefficientIndex];
		Context.Coefficient2 = Context.AdaptationCoefficient2[CoefficientIndex];

		// Populate the block preamble
		//   [0]: Block Predictor
		// [1-2]: Initial Adaptation Delta
		// [3-4]: First Sample
		// [5-6]: Second Sample
		WriteToByteStream<uint8>(CoefficientIndex, EncodedADPCMData, WriteIndex);
		WriteToByteStream<int16>(Context.AdaptationDelta, EncodedADPCMData, WriteIndex);
		WriteToByteStream<int16>(Context.Sample1, EncodedADPCMData, WriteIndex);
		WriteToByteStream<int16>(Context.Sample2, EncodedADPCMData, WriteIndex);

		// Process all the nibble pairs after the preamble.
		while (WriteIndex < BlockSize)
		{
			EncodedADPCMData[WriteIndex] = EncodeNibble(Context, ReadFromArray<int16>(InputPCMSamples, ReadIndex, NumSamples, SampleStride)) << 4;
			EncodedADPCMData[WriteIndex++] |= EncodeNibble(Context, ReadFromArray<int16>(InputPCMSamples, ReadIndex, NumSamples, SampleStride));
		}

		return WriteIndex;
	}

	void Encode(const TArray<uint8>& InputPCMData, TArray<uint8>& CompressedDataStore, const FSoundQualityInfo& QualityInfo)
	{
		const int32 SourceSampleStride = QualityInfo.NumChannels;

		// Input source samples are 2-bytes
		const int32 SourceNumSamples = QualityInfo.SampleDataSize / 2;
		const int32 SourceNumSamplesPerChannel = SourceNumSamples / QualityInfo.NumChannels;

		// Output samples are 4-bits
		const int32 CompressedNumSamplesPerByte = 2;
		const int32 PreambleSamples = 2;
		const int32 BlockSize = 512;
		const int32 PreambleSize = 2 * PreambleSamples + 3;
		const int32 CompressedSamplesPerBlock = (BlockSize - PreambleSize) * CompressedNumSamplesPerByte + PreambleSamples;
		int32 NumBlocksPerChannel = (SourceNumSamplesPerChannel + CompressedSamplesPerBlock - 1) / CompressedSamplesPerBlock;

		const uint32 EncodedADPCMDataSize = NumBlocksPerChannel * BlockSize * QualityInfo.NumChannels;
		uint8* EncodedADPCMData = static_cast<uint8*>(FMemory::Malloc(EncodedADPCMDataSize));
		FMemory::Memzero(EncodedADPCMData, EncodedADPCMDataSize);

		const int16* InputPCMSamples = reinterpret_cast<const int16*>(InputPCMData.GetData());
		uint8* EncodedADPCMChannelData = EncodedADPCMData;

		// Encode each channel, appending channel output as we go.
		for (uint32 ChannelIndex = 0; ChannelIndex < QualityInfo.NumChannels; ++ChannelIndex)
		{
			const int16* ChannelPCMSamples = InputPCMSamples + ChannelIndex;
			int32 SourceSampleOffset = 0;
			int32 DestDataOffset = 0;
			
			for (int32 BlockIndex = 0; BlockIndex < NumBlocksPerChannel; ++BlockIndex)
			{
				EncodeBlock(ChannelPCMSamples + SourceSampleOffset, SourceSampleStride, SourceNumSamples - SourceSampleOffset, BlockSize, EncodedADPCMChannelData + DestDataOffset);
				
				SourceSampleOffset += CompressedSamplesPerBlock * SourceSampleStride;
				DestDataOffset += BlockSize;
			}

			EncodedADPCMChannelData += DestDataOffset;
		}

		ADPCMFormatHeader Format;
		Format.BaseFormat.nChannels = static_cast<uint16>(QualityInfo.NumChannels);
		Format.BaseFormat.nSamplesPerSec = QualityInfo.SampleRate;
		Format.BaseFormat.nBlockAlign = static_cast<uint16>(BlockSize);
		Format.BaseFormat.wBitsPerSample = 4;
		Format.BaseFormat.wFormatTag = WAVE_FORMAT_ADPCM;
		Format.wSamplesPerBlock = static_cast<uint16>(CompressedSamplesPerBlock);
		Format.BaseFormat.nAvgBytesPerSec = ((Format.BaseFormat.nSamplesPerSec / Format.wSamplesPerBlock) * Format.BaseFormat.nBlockAlign);
		Format.wNumCoef = NUM_ADAPTATION_COEFF;
		Format.SamplesPerChannel = SourceNumSamplesPerChannel;
		Format.BaseFormat.cbSize = sizeof(Format) - sizeof(Format.BaseFormat);

		RiffDataChunk RiffDataChunks[2];
		RiffDataChunks[0].ID = UE_MAKEFOURCC('f','m','t',' ');
		RiffDataChunks[0].DataSize = sizeof(Format);
		RiffDataChunks[0].Data = reinterpret_cast<uint8*>(&Format);

		RiffDataChunks[1].ID = UE_MAKEFOURCC('d','a','t','a');
		RiffDataChunks[1].DataSize = EncodedADPCMDataSize;
		RiffDataChunks[1].Data = EncodedADPCMData;

		GenerateWaveFile(RiffDataChunks, CompressedDataStore);
		FMemory::Free(EncodedADPCMData);
	}

	void Encode(const TArray<TArray<uint8> >& InputPCMData, TArray<uint8>& CompressedDataStore, const FSoundQualityInfo& QualityInfo)
	{
		check(InputPCMData.Num() == QualityInfo.NumChannels);

		const int32 SourceSampleStride = 1;

		// Input source samples are 2-bytes
		const int32 SourceNumSamplesPerChannel = (InputPCMData.Num() > 0) ? (InputPCMData[0].Num() / 2) : 0;

		// Output samples are 4-bits
		const int32 CompressedNumSamplesPerByte = 2;
		const int32 PreambleSamples = 2;
		const int32 BlockSize = 512;
		const int32 PreambleSize = 2 * PreambleSamples + 3;
		const int32 CompressedSamplesPerBlock = (BlockSize - PreambleSize) * CompressedNumSamplesPerByte + PreambleSamples;
		int32 NumBlocksPerChannel = (SourceNumSamplesPerChannel + CompressedSamplesPerBlock - 1) / CompressedSamplesPerBlock;

		const uint32 EncodedADPCMDataSize = NumBlocksPerChannel * BlockSize * QualityInfo.NumChannels;
		uint8* EncodedADPCMData = static_cast<uint8*>(FMemory::Malloc(EncodedADPCMDataSize));
		FMemory::Memzero(EncodedADPCMData, EncodedADPCMDataSize);

		uint8* EncodedADPCMChannelData = EncodedADPCMData;

		// Encode each channel, appending channel output as we go.
		for (uint32 ChannelIndex = 0; ChannelIndex < QualityInfo.NumChannels; ++ChannelIndex)
		{
			const int16* ChannelPCMSamples = reinterpret_cast<const int16*>(InputPCMData[ChannelIndex].GetData());

			int32 SourceSampleOffset = 0;
			int32 DestDataOffset = 0;

			for (int32 BlockIndex = 0; BlockIndex < NumBlocksPerChannel; ++BlockIndex)
			{
				const int16* InputPCMSampleOffset = ChannelPCMSamples + SourceSampleOffset;
				const int32 NumBlockSamples = (SourceNumSamplesPerChannel * QualityInfo.NumChannels) - SourceSampleOffset;
				uint8* NewlyEncodedADPCMData = EncodedADPCMChannelData + DestDataOffset;

				EncodeBlock(InputPCMSampleOffset, SourceSampleStride, NumBlockSamples, BlockSize, NewlyEncodedADPCMData);

				SourceSampleOffset += CompressedSamplesPerBlock * SourceSampleStride;
				DestDataOffset += BlockSize;
			}

			EncodedADPCMChannelData += DestDataOffset;
		}

		ADPCMFormatHeader Format;
		Format.BaseFormat.nChannels = static_cast<uint16>(QualityInfo.NumChannels);
		Format.BaseFormat.nSamplesPerSec = QualityInfo.SampleRate;
		Format.BaseFormat.nBlockAlign = static_cast<uint16>(BlockSize);
		Format.BaseFormat.wBitsPerSample = 4;
		Format.BaseFormat.wFormatTag = WAVE_FORMAT_ADPCM;
		Format.wSamplesPerBlock = static_cast<uint16>(CompressedSamplesPerBlock);
		Format.BaseFormat.nAvgBytesPerSec = ((Format.BaseFormat.nSamplesPerSec / Format.wSamplesPerBlock) * Format.BaseFormat.nBlockAlign);
		Format.wNumCoef = NUM_ADAPTATION_COEFF;
		Format.SamplesPerChannel = SourceNumSamplesPerChannel;
		Format.BaseFormat.cbSize = sizeof(Format) - sizeof(Format.BaseFormat);

		RiffDataChunk RiffDataChunks[2];
		RiffDataChunks[0].ID = UE_MAKEFOURCC('f', 'm', 't', ' ');
		RiffDataChunks[0].DataSize = sizeof(Format);
		RiffDataChunks[0].Data = reinterpret_cast<uint8*>(&Format);

		RiffDataChunks[1].ID = UE_MAKEFOURCC('d', 'a', 't', 'a');
		RiffDataChunks[1].DataSize = EncodedADPCMDataSize;
		RiffDataChunks[1].Data = EncodedADPCMData;

		GenerateWaveFile(RiffDataChunks, CompressedDataStore);
		FMemory::Free(EncodedADPCMData);
	}

} // end namespace ADPCM

class FAudioFormatADPCM : public IAudioFormat
{
	enum
	{
		/** Version for ADPCM format, this becomes part of the DDC key. */
		UE_AUDIO_ADPCM_VER = 7,
	};

	void InterleaveBuffers(const TArray<TArray<uint8> >& SrcBuffers, TArray<uint8> & InterleavedBuffer) const
	{
		int32 Channels = SrcBuffers.Num();
		int32 Bytes = SrcBuffers[0].Num();

		InterleavedBuffer.Reserve(Bytes * Channels);

		// Interleave the buffers into one buffer
		int32 CurrentByte = 0;

		while (CurrentByte < Bytes)
		{
			for (const TArray<uint8>& SrcBuffer : SrcBuffers)
			{
				// our data is int16 
				InterleavedBuffer.Push(SrcBuffer[CurrentByte]);
				InterleavedBuffer.Push(SrcBuffer[CurrentByte + 1]);
			}

			CurrentByte += 2;
		}
	}

public:
	virtual bool AllowParallelBuild() const
	{
		return true;
	}

	virtual uint16 GetVersion(FName Format) const override
	{
		check(Format == Audio::NAME_ADPCM || Format == Audio::NAME_PCM);
		return UE_AUDIO_ADPCM_VER;
	}


	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const
	{
		OutFormats.Add(Audio::NAME_ADPCM);
		OutFormats.Add(Audio::NAME_PCM);
	}

	virtual bool Cook(FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& CompressedDataStore) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatADPCM::Cook);
		check(Format == Audio::NAME_ADPCM || Format == Audio::NAME_PCM);

		if (QualityInfo.Quality == 100)
		{
			LPCM::Encode(SrcBuffer, CompressedDataStore, QualityInfo);
		}
		else
		{
			ADPCM::Encode(SrcBuffer, CompressedDataStore, QualityInfo);
		}
		
		return CompressedDataStore.Num() > 0;
	}

	virtual bool CookSurround(FName Format, const TArray<TArray<uint8> >& SrcBuffers, FSoundQualityInfo& QualityInfo, TArray<uint8>& CompressedDataStore) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatADPCM::CookSurround);

		// Ensure the right format
		check(Format == Audio::NAME_ADPCM || Format == Audio::NAME_PCM);
		// Ensure at least two channel
		check(SrcBuffers.Num() > 1);
		// Ensure one buffer per channel
		check(SrcBuffers.Num() == QualityInfo.NumChannels);
		// Ensure even number of bytes (data is int16)
		check((SrcBuffers[0].Num() % 1) == 0);

		if (QualityInfo.Quality == 100)
		{
			TArray<uint8> InterleavedSrc;
			
			InterleaveBuffers(SrcBuffers, InterleavedSrc);
			LPCM::Encode(InterleavedSrc, CompressedDataStore, QualityInfo);
		}
		else
		{
			ADPCM::Encode(SrcBuffers, CompressedDataStore, QualityInfo);
		}
		
		return true;
	}

	virtual int32 Recompress(FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& OutBuffer) const
	{
		check(Format == Audio::NAME_ADPCM || Format == Audio::NAME_PCM);

		// Recompress is only necessary during editor previews
		return 0;
	}

	virtual int32 GetMinimumSizeForInitialChunk(FName Format, const TArray<uint8>& SrcBuffer) const override
	{
		uint8 const*	SrcData = SrcBuffer.GetData();
		int32			SrcSize = SrcBuffer.Num();
		uint32			BytesProcessed = 0;

		FWaveModInfo	WaveInfo;
		WaveInfo.ReadWaveInfo((uint8*)SrcData, SrcSize);

		return WaveInfo.SampleDataStart - SrcData;
	}

	virtual bool SplitDataForStreaming(const TArray<uint8>& SrcBuffer, TArray<TArray<uint8>>& OutBuffers, const int32 InitialMaxChunkSize, const int32 MaxChunkSize) const override
	{
		uint8 const*	SrcData = SrcBuffer.GetData();
		int32			SrcSize = SrcBuffer.Num();
		uint32			BytesProcessed = 0;
		
		FWaveModInfo	WaveInfo;
		WaveInfo.ReadWaveInfo((uint8*)SrcData, SrcSize);
		
		// Choose a chunk size that is much larger then the number of samples that the os audio render callback will ask for so that the chunk system has time to load new data before its needed
		// The audio render callback will typically ask for a power of 2 number of samples. However, there is not a power of 2 number of samples in the uncompressed block.
		// The ADPCM block size is 512 bytes which will contain 1012 samples (since the first 2 samples of the block are uncompressed and there are some preamble bytes).
		// Going with MONO_PCM_BUFFER_SIZE bytes per chunk should ensure that new sample data will always be available for the audio render callback
		// the audio render callback will not be an even multiple of uncompressed samples
		// Also, the first 78 odd bytes of the buffer is for the header so the first chunk is 78 bytes bigger then the rest. This needs to be accounted for when using chunk 0.

		// The incoming data is organized by channel (all samples for channel 0 then all samples for channel 1, etc) but for streaming we need the channels interlaced by compressed blocks
		// so that each chunk contains an even number of compressed blocks per channel, ie channels can not span one chunk
		
		const int32	NumChannels = *WaveInfo.pChannels;
		
		if(*WaveInfo.pFormatTag == WAVE_FORMAT_ADPCM)
		{
			int32 BlockSize = *WaveInfo.pBlockAlign;
			const int32 NumBlocksPerChannel = (WaveInfo.SampleDataSize + BlockSize * NumChannels - 1) / (BlockSize * NumChannels);

			// Add a new chunk for the header
			int32 HeaderSize = WaveInfo.SampleDataStart - SrcData;

			check(InitialMaxChunkSize >= HeaderSize);
			AddNewChunk(OutBuffers, InitialMaxChunkSize);
			AddChunkData(OutBuffers, SrcData, HeaderSize);
			int32 CurChunkDataSize = HeaderSize;

			// Start looping through the rest of the chunks:
			int32 ThisChunkMaxSize = InitialMaxChunkSize;

			for (int32 BlockIndex = 0; BlockIndex < NumBlocksPerChannel; ++BlockIndex)
			{
				// If the next block size will put us over the max chunk size, then add a new chunk
				if (CurChunkDataSize + NumChannels * BlockSize >= ThisChunkMaxSize)
				{
					// Start a new chunk with the reserve memory for the max chunk size
					AddNewChunk(OutBuffers, MaxChunkSize);
					ThisChunkMaxSize = MaxChunkSize;
					CurChunkDataSize = 0;
				}

				// Always add chunks in NumChannels pairs
				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					// Compute the offset of this block from the non-interleaved input data
					const int32 Offset = (ChannelIndex * NumBlocksPerChannel + BlockIndex) * BlockSize;;
					const uint8* DataPtr = WaveInfo.SampleDataStart + Offset;
					AddChunkData(OutBuffers, DataPtr, BlockSize);
					CurChunkDataSize += BlockSize;
				}
			}
		}
		else if(*WaveInfo.pFormatTag == WAVE_FORMAT_LPCM)
		{
			int32 FrameSize = sizeof(uint16) * NumChannels;

			// Add the first chunk and the header data
			AddNewChunk(OutBuffers, InitialMaxChunkSize);	

			// Add the header data
			int32 HeaderSize = WaveInfo.SampleDataStart - SrcData;
			check(InitialMaxChunkSize >= HeaderSize);
			AddChunkData(OutBuffers, SrcData, HeaderSize);

			SrcSize -= HeaderSize;
			SrcData = WaveInfo.SampleDataStart;

			int32 DataLeftInCurChunk = InitialMaxChunkSize - HeaderSize;

			while (SrcSize > 0)
			{
				// Calculate how many frames can fit in whats left of the current chunk
				DataLeftInCurChunk = FMath::Min(DataLeftInCurChunk, SrcSize);

				int32 NumFramesInNextChunk = DataLeftInCurChunk / FrameSize;
				int32 SizeOfNewChunk = NumChannels * NumFramesInNextChunk * sizeof(uint16);

				AddChunkData(OutBuffers, SrcData, SizeOfNewChunk);

				SrcSize -= SizeOfNewChunk;
				SrcData += SizeOfNewChunk;

				
				if (SrcSize > 0)
				{
					DataLeftInCurChunk = MaxChunkSize;
					AddNewChunk(OutBuffers, MaxChunkSize);
				}
			}
		}
		else
		{
			return false;
		}
		
		return true;
	}
	
	// Add a new chunk and reserve ChunkSize bytes in it
	void AddNewChunk(TArray<TArray<uint8>>& OutBuffers, int32 ChunkReserveSize) const
	{
		TArray<uint8>& NewBuffer = OutBuffers.AddDefaulted_GetRef();
		NewBuffer.Empty(ChunkReserveSize);
	}
	
	// Add data to the current chunk
	void AddChunkData(TArray<TArray<uint8>>& OutBuffers, const uint8* ChunkData, int32 ChunkDataSize) const
	{
		TArray<uint8>& TargetBuffer = OutBuffers[OutBuffers.Num() - 1];
		TargetBuffer.Append(ChunkData, ChunkDataSize);
	}
};


/**
 * Module for ADPCM audio compression
 */

static IAudioFormat* Singleton = NULL;

class FAudioPlatformADPCMModule : public IAudioFormatModule
{
public:
	virtual ~FAudioPlatformADPCMModule()
	{
		delete Singleton;
		Singleton = NULL;
	}
	virtual IAudioFormat* GetAudioFormat()
	{
		if (!Singleton)
		{
			/* TODO::JTM - Dec 10, 2012 09:55AM - Library initialization */
			Singleton = new FAudioFormatADPCM();
		}
		return Singleton;
	}
};

IMPLEMENT_MODULE(FAudioPlatformADPCMModule, AudioFormatADPCM);
